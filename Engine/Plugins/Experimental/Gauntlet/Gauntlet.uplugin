{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Gauntlet",
	"Description": "Provides a helper class for creating and managing tests in your game",
	"Category": "Testing",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "Gauntlet",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	],
	"IsExperimentalVersion": false
}