// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class Gauntlet : ModuleRules
	{
		public Gauntlet(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine"
				});

			if (Target.bCompileAgainstEditor)
			{
				PublicDependencyModuleNames.AddRange(new string[]
				{
				"UnrealEd"
				});
			}

			PublicIncludePaths.AddRange(
				new string[]
				{
				}
			);
		}
	}
}