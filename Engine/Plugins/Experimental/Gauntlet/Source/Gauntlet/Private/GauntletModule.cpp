// Copyright Epic Games, Inc. All Rights Reserved.
#include "GauntletModule.h"
#include "Engine/World.h"
#include "UObject/Package.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CommandLine.h"
#include "GauntletTestController.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectIterator.h"
#include "Containers/Ticker.h"
#include "GameFramework/GameStateBase.h"
#include "UnrealClient.h"
#if WITH_EDITOR
#include "Editor.h"
#endif


class GAUNTLET_API FGauntletModuleImpl
	: public FGauntletModule
	, public FGCObject

{
public:
	/** IModuleInterface interface */
	void StartupModule() override;
	void ShutdownModule() override;

	/** GauntletModule interface */
	void	SetGameStateToTestStateMapping(const TMap<UClass*, FName>& Mapping) override;
	void	SetWorldToTestStateMapping(const TMap<FString, FName>& Mapping) override;
	void	BroadcastStateChange(FName NewState) override;
	FName	GetCurrentState() const override { return CurrentState; }
	double	GetTimeInCurrentState() const override { return TimeInCurrentState; }

	FString			GetCurrentMap() const { return CurrentMap; }

	void			SetScreenshotPeriod(float Period) { ScreenshotPeriod = Period; }

	void			MarkHeartbeatActive(const FString& OptionalStatusMessage = FString());

	UGauntletTestController*	GetTestController(UClass* ControllerClass) override;

protected:

	/**
	* FGCObject interface - add references to our UObject-inherited controllers to prevent
	* them being collected
	*/
	void		AddReferencedObjects(FReferenceCollector& Collector) override;
	FString		GetReferencerName() const override
	{
		return TEXT("FGauntletModuleImpl");
	}

	/** Handler for PostMapChange delegate */
	void		InnerPostMapChange(UWorld* World);

	/** Handler for PreMapChange delegate */
	void		InnerPreMapChange(const FString& MapName);

	/** Tick callback */
	void		InnerTick(const float TimeDelta);

	/** Performs actual initialization */
	void		PerformInitialization();

	/** Loads all controllers based on command line arts */
	void		LoadControllers();

	void		TakeScreenshot();

	/** Prints a heartbeat message to the log */
	void		LogHeartbeat();

private:

	/** Handle to our tick callback */
	FTSTicker::FDelegateHandle					TickHandle;

	/** Currently active controllers */
	TArray<TObjectPtr<UGauntletTestController>>	Controllers;

	/** True if a state has been set */
	bool							StateSet;

	FName							CurrentState;
	double							TimeInCurrentState;
	FString							CurrentMap;
	TMap<UClass*, FName>			GameStateStateMap;
	TMap<FString, FName>			MapGameStateMap;
	UClass*							CurrentGameStateClass;

	double							LastScreenshotTime;
	float							ScreenshotPeriod;

	/**
	 * Heartbeat Internal Members
	 */

	/** True if the next heartbeat is marked Active rather than Idle */
	bool bHeartbeatActive;
	/** How often to regularly log heartbeat messages, in seconds */
	float HeartbeatPeriod;
	/** Platform time in seconds of the last heartbeat. Used with HeartbeatPeriod. */
	float LastHeartbeatTime;
	/** Optional status message to include with Active heartbeats */
	FString HeartbeatStatus;
};


IMPLEMENT_MODULE(FGauntletModuleImpl, Gauntlet)

DEFINE_LOG_CATEGORY(LogGauntlet);

FName FGauntletStates::Initialized = TEXT("Gauntlet_Initialized");


void FGauntletModuleImpl::StartupModule()
{
	StateSet = false;
	CurrentGameStateClass = nullptr;
	TimeInCurrentState = 0;
	LastScreenshotTime = 0.0f;
	ScreenshotPeriod = 0.0f;
	bHeartbeatActive = false;
	HeartbeatPeriod = 0.0f;
	LastHeartbeatTime = 0.f;
	HeartbeatStatus = FString();

	if (IsRunningGame() || IsRunningDedicatedServer())
	{
		UE_LOGF(LogGauntlet, Display, "Gauntlet Initialized");
		FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FGauntletModuleImpl::PerformInitialization);
	}
#if WITH_EDITOR
	else
	{
		FEditorDelegates::PreBeginPIE.AddLambda([this](bool bIsSimulating){ this->PerformInitialization(); });
	}
#endif
}

void FGauntletModuleImpl::PerformInitialization()
{
	if (Controllers.Num() == 0)
	{
		FCoreUObjectDelegates::PostLoadMapWithWorld.AddRaw(this, &FGauntletModuleImpl::InnerPostMapChange);
		FCoreUObjectDelegates::PreLoadMap.AddRaw(this, &FGauntletModuleImpl::InnerPreMapChange);

		FParse::Value(FCommandLine::Get(), TEXT("gauntlet.screenshotperiod="), ScreenshotPeriod);
		FParse::Value(FCommandLine::Get(), TEXT("gauntlet.heartbeatperiod="), HeartbeatPeriod);

		LoadControllers();

		float kTickRate = 1.0f;
		FParse::Value(FCommandLine::Get(), TEXT("gauntlet.tickrate="), kTickRate);


		TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this, kTickRate](float TimeDelta)
		{
			// ticker passes in frame-delta, not tick delta...
			InnerTick(kTickRate);
			return true;
		}),
		kTickRate);
	}
}

void FGauntletModuleImpl::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	FCoreUObjectDelegates::PreLoadMap.RemoveAll(this);
	FCoreUObjectDelegates::PostLoadMapWithWorld.RemoveAll(this);

	if (TickHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickHandle);
		TickHandle.Reset();
	}

	UE_LOGF(LogGauntlet, Log, "Gauntlet Shutdown");

}

void FGauntletModuleImpl::LoadControllers()
{
	FString ControllerString;

	FParse::Value(FCommandLine::Get(), TEXT("gauntlet="), ControllerString, false);

	if (ControllerString.Len())
	{
		TArray<FString> ControllerNames;

		ControllerString.ParseIntoArrayWS(ControllerNames, TEXT(","));

		TSet<FString> ControllersToCreate;

		for (const FString& Name : ControllerNames)
		{
			UClass* TestClass = nullptr;

			for (TObjectIterator<UClass> It; It; ++It)
			{
				UClass* Class = *It;

				FString FullName = Name;

				// Search for SomethingTestController and ControllerSomethingTest. The last is legacy
				FString PartialName1 = Name + TEXT("Controller");
				FString PartialName2 = FString(TEXT("Controller")) + Name;

				if (Class->IsChildOf<UGauntletTestController>())
				{
					FString ClassName = Class->GetName();
					if (ClassName == FullName || ClassName.EndsWith(PartialName1) || ClassName.EndsWith(PartialName2))
					{
						// Gauntlet has a couple of test classes, so need to differentiate between "GauntletFooTest" and "GameFooTest".
						// that will both be launched via -gauntlet=FooTest
						bool GauntletDefault = ClassName.StartsWith(TEXT("Gauntlet"));

						TestClass = Class;

						// If not gauntlet stop searching
						if (!GauntletDefault)
						{
							break;
						}
					}
				}
			}			

			checkf(TestClass, TEXT("Could not find class for controller %s"), *Name);

			UGauntletTestController* NewController = NewObject<UGauntletTestController>(GetTransientPackage(), TestClass);

			check(NewController);

			UE_LOGF(LogGauntlet, Display, "Added Gauntlet controller %ls", *Name);

			// Important - add the controller first! Some controllers may trigger GC's which would
			// then result in them being collected...
			Controllers.Add(NewController);
		}
	}

	for (auto Controller : Controllers)
	{
		Controller->OnInit();
	}
}


void FGauntletModuleImpl::SetGameStateToTestStateMapping(const TMap<UClass*, FName>& Mapping)
{
	GameStateStateMap  = Mapping;
}

void FGauntletModuleImpl::SetWorldToTestStateMapping(const TMap<FString, FName>& Mapping)
{
	MapGameStateMap = Mapping;
}

void FGauntletModuleImpl::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (auto& Controller : Controllers)
	{
		Collector.AddReferencedObject(Controller);
	}
}

void FGauntletModuleImpl::BroadcastStateChange(FName NewState)
{
	if (NewState == CurrentState && StateSet)
	{
		return;
	}

	for (auto Controller : Controllers)
	{
		Controller->OnStateChange(CurrentState, NewState);
	}

	CurrentState = NewState;
	StateSet = true;
	TimeInCurrentState = 0.0;

	UE_LOGF(LogGauntlet, Display, "Changed state to %ls", *NewState.ToString());
}

void FGauntletModuleImpl::InnerPreMapChange(const FString& MapName)
{
	for (auto Controller : Controllers)
	{
		Controller->OnPreMapChange();
	}
}

void FGauntletModuleImpl::InnerPostMapChange(UWorld* World)
{
	if (!World)
	{
		// Failed to load requested map
		for (auto Controller : Controllers)
		{
			Controller->OnPostMapChange(World);
		}

		return;
	}

	CurrentMap = World->GetMapName();

	for (auto Controller : Controllers)
	{
		Controller->OnPostMapChange(World);
	}

	// process mappings
	if (MapGameStateMap.Contains(CurrentMap))
	{
		BroadcastStateChange(MapGameStateMap[CurrentMap]);
	}
}

// FTickableGameObject interface
void FGauntletModuleImpl::InnerTick(const float TimeDelta)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FGauntletModuleImpl_InnerTick);
	
	UWorld* World = GWorld;

#if WITH_EDITOR
	UEditorEngine* EditorEngine = Cast<UEditorEngine>(GEngine);
	if (GIsEditor && EditorEngine != nullptr && EditorEngine->PlayWorld != nullptr)
	{
		// lets use PlayWorld during PIE/Simulate
		World = EditorEngine->PlayWorld;
	}
#endif

    if (!World)
    {
        return;
    }

    if (AGameStateBase* GameState = World->GetGameState())
	{
		if (GameState->GetClass() != CurrentGameStateClass)
		{
			CurrentGameStateClass = GameState->GetClass();

			FName NextState;

			for (const auto& KP : GameStateStateMap)
			{
				if (CurrentGameStateClass->IsChildOf(KP.Key))
				{
					NextState = KP.Value;

					if (CurrentGameStateClass == KP.Key)
					{
						break;
					}
				}
			}

			if (!NextState.IsNone())
			{
				BroadcastStateChange(NextState);
			}
		}

		if (ScreenshotPeriod > 0.0f && (FPlatformTime::Seconds() - LastScreenshotTime) > ScreenshotPeriod)
		{
			TakeScreenshot();
			LastScreenshotTime = FPlatformTime::Seconds();
		}

		if (HeartbeatPeriod > 0.0f && (FPlatformTime::Seconds() - LastHeartbeatTime) > HeartbeatPeriod)
		{
			LogHeartbeat();
		}
	}

	TimeInCurrentState += TimeDelta;

	for (auto Controller : Controllers)
	{
		Controller->OnTick(TimeDelta);
	}
}

void FGauntletModuleImpl::TakeScreenshot()
{
	FScreenshotRequest::RequestScreenshot(true);
}

void FGauntletModuleImpl::LogHeartbeat()
{
	FString HeartbeatMessage = bHeartbeatActive ? FString("GauntletHeartbeat: Active") : FString("GauntletHeartbeat: Idle");
	UE_LOGF(LogGauntlet, Display, "%ls %ls", *HeartbeatMessage, *HeartbeatStatus);
	bHeartbeatActive = false;
	HeartbeatStatus.Empty();
	LastHeartbeatTime = FPlatformTime::Seconds();
}

void FGauntletModuleImpl::MarkHeartbeatActive(const FString& OptionalStatusMessage /*= FString()*/)
{
	bHeartbeatActive = true;
	// If a status message is given, heartbeat immediately so that the message is printed at the relevant time.
	if (!OptionalStatusMessage.IsEmpty())
	{
		HeartbeatStatus = OptionalStatusMessage;
		LogHeartbeat();
	}
}

UGauntletTestController* FGauntletModuleImpl::GetTestController(UClass* ControllerClass)
{
	for (auto Controller : Controllers)
	{
		if (Controller->IsA(ControllerClass))
		{
			return Controller;
		}
	}

	return nullptr;
}
