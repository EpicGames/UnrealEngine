// Copyright Epic Games, Inc. All Rights Reserved.

#include "GauntletTestController.h"
#include "Engine/World.h"
#include "GauntletModule.h"
#include "Modules/ModuleManager.h"
#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GauntletTestController)


UGauntletTestController::UGauntletTestController(const FObjectInitializer& ObjectInitializer)
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		ParentModule = &FModuleManager::Get().GetModuleChecked<FGauntletModule>(TEXT("Gauntlet"));
	}
}

UGauntletTestController::~UGauntletTestController()
{

}

FName UGauntletTestController::GetCurrentState() const
{
	check(ParentModule);
	return ParentModule->GetCurrentState();
}

double UGauntletTestController::GetTimeInCurrentState() const
{
	check(ParentModule);
	return ParentModule->GetTimeInCurrentState();
}

FString UGauntletTestController::GetCurrentMap() const
{
	return GetWorld()->GetName();
}

/**
* Returns the gauntlet module running this test
*/
FGauntletModule* UGauntletTestController::GetGauntlet()
{
	return ParentModule;
}

void UGauntletTestController::MarkHeartbeatActive(const FString& OptionalStatusMessage /*= FString()*/)
{
	FString StatusMessage = OptionalStatusMessage;
	if (!StatusMessage.IsEmpty())
	{
		StatusMessage = FString::Printf(TEXT("[%s] %s"), *GetName(), *StatusMessage);
	}

	GetGauntlet()->MarkHeartbeatActive(StatusMessage);
}

UWorld* UGauntletTestController::GetWorld() const
{
#if WITH_EDITOR
	UEditorEngine* EditorEngine = Cast<UEditorEngine>(GEngine);
	if (GIsEditor && EditorEngine != nullptr && EditorEngine->PlayWorld != nullptr)
	{
		// lets use PlayWorld during PIE/Simulate
		return EditorEngine->PlayWorld;
	}
#endif
	return GWorld;
}

APlayerController* UGauntletTestController::GetFirstPlayerController() const
{
	return GetWorld() ? GetWorld()->GetFirstPlayerController() : nullptr;	
}

void UGauntletTestController::EndTest(int32 ExitCode /*= 0*/)
{
	UE_LOGF(LogGauntlet, Display, "**** TEST COMPLETE. EXIT CODE: %d ****", ExitCode);
	// we flush logs because we don't (currently...) want to treat shutdown errors as failures
	GLog->FlushThreadedLogs();
#if WITH_EDITOR
	if (GIsEditor)
	{
		GEngine->DeferredCommands.Add(TEXT("QUIT_EDITOR"));
		return;
	}
#endif
	// force exit only if platform doesn't support quitting
	FPlatformMisc::RequestExitWithStatus(!FPlatformProperties::SupportsQuit(), static_cast<uint8>(ExitCode));
}

