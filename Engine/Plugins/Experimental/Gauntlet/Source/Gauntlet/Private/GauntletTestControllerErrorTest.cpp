// Copyright Epic Games, Inc. All Rights Reserved.

#include "GauntletTestControllerErrorTest.h"
#include "GauntletModule.h"
#include "Misc/CommandLine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GauntletTestControllerErrorTest)


void UGauntletTestControllerErrorTest::OnInit()
{
	ErrorDelay = 0;
	ErrorType = TEXT("check");

	FParse::Value(FCommandLine::Get(), TEXT("errortest.delay="), ErrorDelay);
	FParse::Value(FCommandLine::Get(), TEXT("errortest.type="), ErrorType);
}

void UGauntletTestControllerErrorTest::OnTick(float TimeDelta)
{
	if (GetTimeInCurrentState() > ErrorDelay)
	{
		if (ErrorType == TEXT("ensure"))
		{
			UE_LOGF(LogGauntlet, Display, "Issuing ensure as requested");
			ensureMsgf(false, TEXT("Ensuring false...."));
			EndTest(-1);
		}
		else if (ErrorType == TEXT("check"))
		{
			UE_LOGF(LogGauntlet, Display, "Issuing failed check as requested");
			checkf(false, TEXT("Asserting as requested"));
		}		
		else if (ErrorType == TEXT("fatal"))
		{
			UE_LOGF(LogGauntlet, Fatal, "Issuing fatal error as requested");
		}
		else if (ErrorType == TEXT("gpf"))
		{
			UE_LOGF(LogGauntlet, Display, "Issuing GPF as requested");
			int* Ptr = (int*)0;
			CA_SUPPRESS(6011);
			*Ptr = 42; //-V522
		}
		else
		{
			UE_LOGF(LogGauntlet, Error, "No recognized error request. Failing test");
			EndTest(-1);
		}
	}
}


