// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Modules/ModuleInterface.h"
#include "Logging/LogMacros.h"

#define UE_API GAUNTLET_API

/**
*	Gauntlet states. Define your own states by inheriting from this or some other
*	form that you see fit.
*
*/
struct FGauntletStates
{
	static UE_API FName Initialized;
};

class UGauntletTestController;

/**
*	Main Gauntlet Module. This is module is responsible for managing its underlying controllers and
*	propagating events and state changes to them as necessary.
*	
*	After initialization you should be prepared to feed this module with states that your controllers
*	can respond to. This can be done either manually (BroadcastStateChange), through one of the helpers
*	(e.g. SetGameStateToTestStateMapping), or a combination of both.
*/
class FGauntletModule : public IModuleInterface
{
public:

	/**
	*
	*	This is a convenient way of binding a list of AGameState  types to your own state defines. When
	*	there is a state change in the world (e.g. from loading a new map) the mapped state type will be
	*	broadcast to all controllers.
	*
	* @param: 	Mapping - A map e.g. [AGameStateSomethingClass, StateSomething]
	* @return: void
	*/
	virtual void			SetGameStateToTestStateMapping(const TMap<UClass*, FName>& Mapping) = 0;

	/**
	*
	*	This is a convenient way of binding a list of maps types to your own state defines. When
	*	there is a state change in the world (e.g. from loading a new map) the mapped state type will be
	*	broadcast to all controllers.
	*
	* @param: 	Mapping - A map e.g. [TEXT("FrontendMap"), StateFrontend]
	* @return: void
	*/
	virtual void			SetWorldToTestStateMapping(const TMap<FString, FName>& Mapping) = 0;

	/**
	*	Manually Broadcasts a state change to all current Gauntlet controllers. This can be used as an alternative
	*	or addition to the functions above for broadcasting state changes to running controllers.
	*
	* @param: 	FName		New State
	* @return: void
	*/
	virtual void			BroadcastStateChange(FName NewState) = 0;

	/**
	* Returns the current  state
	*
	* @return: FName
	*/
	virtual FName			GetCurrentState() const = 0;

	/**
	* Returns the time spent in the current state
	*
	* @return: double
	*/
	virtual double			GetTimeInCurrentState() const = 0;

	/**
	 * Sets the rate for screenshots to be taken (default = 0)
	 */
	virtual void			SetScreenshotPeriod(float Period) = 0;

	/**
	 * Mark the next heartbeat as active. If a status message is given, immediately log an active heartbeat with that message.
	 */
	virtual void			MarkHeartbeatActive(const FString& OptionalStatusMessage = FString()) = 0;

	/**
	 * Returns the first controller (if any) matching the provided name
	 */
	virtual UGauntletTestController*	GetTestController(UClass* ControllerClass) = 0;

	/**
	* Templated casty version of the above
	*/
	template< class T >
	inline T* GetTestController()
	{
		return static_cast<T*>(GetTestController(T::StaticClass()));
	}
};



/** Logging category */
GAUNTLET_API DECLARE_LOG_CATEGORY_EXTERN(LogGauntlet, Log, All);

#undef UE_API
