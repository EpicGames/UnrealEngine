// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GauntletTestController.generated.h"

#define UE_API GAUNTLET_API

class FGauntletModule;

class APlayerController;

/**
 *	Base class for games to implement test controllers that use the Gauntlet native
 *	framework. This is a very thin class that is created automatically based on 
 *	command line params (-gauntlet=MyControllerName) and provides easily overridden
 *	functions that represent state changes and ticking
 *
 *	In essence your derived class should implement logic that starts and monitors
 *	a test, then calls EndTest(Result) when the desired criteria are met (or not!)
 */
UCLASS(MinimalAPI)
class UGauntletTestController : public UObject
{
	GENERATED_BODY()

	friend class FGauntletModule;

public:

	/** Default constructur */
	UE_API UGauntletTestController(const FObjectInitializer& ObjectInitializer);
	UE_API virtual ~UGauntletTestController();

	// Overridable delegates for some of the most useful test points

	/**
	 *	Called when the controller is first initialized
	 */
	virtual void	OnInit() {}

	/**
	 *	Called prior to a map change
	 */
	virtual void	OnPreMapChange() {}

	/**
	 *	Called after a map change. GetCurrentMap() will now return the new map
	 */
	virtual void	OnPostMapChange(UWorld* World) {}

	/**
	 *	Called periodically to let the controller check and control state
	 */
	virtual void	OnTick(float TimeDelta) {}

	/**
	 *	Called when a state change is applied to the module. States are game-driven.
	 *	GetCurrentState() == OldState until this function returns
	 */
	virtual void	OnStateChange(FName OldState, FName NewState) {}

	/**
	 * Return the current world
	 */
	UE_API UWorld* GetWorld() const override;

	/**
	 *	Helper function that returns the first player controller in the world (may be null depending on when called).
	 */
	UE_API APlayerController* GetFirstPlayerController() const;

protected:

	/**
	 *	Returns the current state applied to Gauntlet
	 */
	UE_API FName			GetCurrentState() const;

	/**
	 *	Return the time since OnStateChange was called with the current state
	 */
	UE_API double			GetTimeInCurrentState() const;

	/**
	 *	Return the name of the current persistent map
	 */
	UE_API FString			GetCurrentMap() const;

	/**
	 *	Called to end testing and exit the app with provided code, static to avoid test instance state/lifetime dependency
	 */
	static UE_API void	EndTest(int32 ExitCode = 0);

	/**
	 * Returns the gauntlet module running this test
	 */
	UE_API FGauntletModule* GetGauntlet();

	/**
	 * Marks the next heartbeat as active and immediately forces a heartbeat with the given status message if one is specified.
	 * Test controllers should call this regularly as purposeful actions happens or meaningful states are achieved within the test.
	 */
	UE_API void MarkHeartbeatActive(const FString& OptionalStatusMessage = FString());

private:

	FGauntletModule*  ParentModule;

};

#undef UE_API
