// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GauntletTestController.h"
#include "GauntletTestControllerBootTest.generated.h"

#define UE_API GAUNTLET_API

UCLASS(MinimalAPI)
class UGauntletTestControllerBootTest : public UGauntletTestController
{
	GENERATED_BODY()

protected:

	virtual bool IsBootProcessComplete() const { return false; }
	UE_API virtual void OnTick(float TimeDelta) override;
};

#undef UE_API
