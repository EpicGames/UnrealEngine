// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GauntletTestController.h"
#include "GauntletTestControllerErrorTest.generated.h"

#define UE_API GAUNTLET_API

UCLASS(MinimalAPI)
class UGauntletTestControllerErrorTest : public UGauntletTestController
{
	GENERATED_BODY()

protected:

	float			ErrorDelay;
	FString			ErrorType;
	bool			RunOnServer;

	UE_API void	OnInit() override;
	UE_API void	OnTick(float TimeDelta)		override;
};

#undef UE_API
